// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PixWinPlugin_WinGDK : PixWinPlugin
	{
		public PixWinPlugin_WinGDK( ReadOnlyTargetRules Target) : base(Target)
		{
		}
	}
}