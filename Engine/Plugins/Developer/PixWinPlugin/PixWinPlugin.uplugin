{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "PIX on Windows GPU Capture Plugin",
	"Description": "PIX on Windows graphics debugger integration.",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"EnabledByDefault": true,
	"SupportedTargetPlatforms":  [ "Win64" ],
	"Modules": [
		{
			"Name": "PixWinPlugin",
			"Type": "DeveloperTool",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64" ],
			"TargetDenyList": [ "Server" ]
		}
	]
}
