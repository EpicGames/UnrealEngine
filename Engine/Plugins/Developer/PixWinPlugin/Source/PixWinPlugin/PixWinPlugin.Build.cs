// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	[SupportedPlatformGroups("Windows")]
	public class PixWinPlugin : ModuleRules
	{
		public PixWinPlugin(ReadOnlyTargetRules Target) : base(Target)
        {
			PublicDependencyModuleNames.AddAll(
				"InputDevice",
				"RenderCore"
			);

			PrivateDependencyModuleNames.AddAll(
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Projects",
				"RHI"
			);

			if (Target.Configuration != UnrealTargetConfiguration.Shipping)
			{
				PrivateDependencyModuleNames.Add("WinPixEventRuntime");
			}

			if (Target.bBuildEditor == true)
			{
				DynamicallyLoadedModuleNames.Add("LevelEditor");

				PrivateDependencyModuleNames.AddAll(
					"Slate"
					, "SlateCore"
					, "EditorFramework"
					, "UnrealEd"
					, "MainFrame"
					, "GameProjectGeneration"
					, "ToolMenus"
				);
			}
		}
	}
}
