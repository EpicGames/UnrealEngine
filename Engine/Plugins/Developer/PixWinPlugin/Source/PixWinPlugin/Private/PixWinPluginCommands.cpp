// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PixWinPluginCommands.h"

#define LOCTEXT_NAMESPACE "PixWinPlugin"

void FPixWinPluginCommands::RegisterCommands()
{
	UI_COMMAND(CaptureFrame, "Capture Frame", "Captures the next frame and launches the PIX UI if not attached (Alt+F12)", EUserInterfaceActionType::Button, FInputChord(EKeys::F12, EModifierKey::Alt));
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_EDITOR
