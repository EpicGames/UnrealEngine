// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "PixWinPluginStyle.h"
#include "Framework/Commands/Commands.h"

class FPixWinPluginCommands : public TCommands<FPixWinPluginCommands>
{
public:
	FPixWinPluginCommands()
		: TCommands<FPixWinPluginCommands>(TEXT("PixWinPlugin"), NSLOCTEXT("Contexts", "PixWinPlugin", "PixWin Plugin"), NAME_None, FPixWinPluginStyle::Get()->GetStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

	TSharedPtr<class FUICommandInfo> CaptureFrame;
};

#endif
