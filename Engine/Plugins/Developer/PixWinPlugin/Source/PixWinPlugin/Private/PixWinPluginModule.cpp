// Copyright Epic Games, Inc. All Rights Reserved.

#include "PixWinPluginModule.h"

#include "RenderingThread.h"
#include "RHI.h"
#include "UnrealClient.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

#if WITH_EDITOR
#include "Framework/Application/SlateApplication.h"
#include "Editor.h"
#include "EditorViewportClient.h"
#include "SPixWinPluginEditorExtension.h"
#endif

#if !defined(WITH_PIX_EVENT_RUNTIME)
#define WITH_PIX_EVENT_RUNTIME 0
#endif

#define PIX_PLUGIN_ENABLED (WITH_PIX_EVENT_RUNTIME && !UE_BUILD_SHIPPING)

#if PIX_PLUGIN_ENABLED
#define USE_PIX 1
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
	#include <pix3.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#endif

DEFINE_LOG_CATEGORY(PixWinPlugin);

#define LOCTEXT_NAMESPACE "PixWinPlugin"

static FString MakeWinPixCaptureFilePath(const FString& InFilename)
{
	FString Filename = InFilename;
	if (Filename.IsEmpty())
	{
		const FDateTime DateTime = FDateTime::Now();
		Filename = FString(TEXT("UEPixCapture_")) + DateTime.ToString(TEXT("%Y.%m.%d-%H.%M.%S.%sZ"));
	}

	const bool bAbsoluteFileName = !FPaths::IsRelative(Filename);
	FString FileName = bAbsoluteFileName ? Filename : FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() / FString("PixCaptures") / Filename);
	FileName = FPaths::SetExtension(FileName, "wpix");
	FPaths::MakePlatformFilename(FileName);
	return FileName;
}

namespace Impl
{
#if PIX_PLUGIN_ENABLED
	/** Container for graphics analysis com interface. */
	class FPixGraphicsAnalysisInterface
	{
	public:
		FPixGraphicsAnalysisInterface()
		{
			WinPixGpuCapturerHandle = FPlatformProcess::GetDllHandle(L"WinPixGpuCapturer.dll");

			if (!WinPixGpuCapturerHandle)
			{
				if (FParse::Param(FCommandLine::Get(), TEXT("attachPIX")))
				{
					WinPixGpuCapturerHandle = PIXLoadLatestWinPixGpuCapturerLibrary();
				}
			}

			if (WinPixGpuCapturerHandle)
			{
				PIXSetHUDOptions(PIX_HUD_SHOW_ON_NO_WINDOWS);
			}
		}

		bool IsValid()
		{
			return WinPixGpuCapturerHandle != nullptr;
		}

		bool IsAttached()
		{
			return WinPixGpuCapturerHandle != nullptr && PIXIsAttachedForGpuCapture();
		}

		void BeginCapture(HWND WindowHandle, const FString& DestFileName)
		{
			if (WinPixGpuCapturerHandle)
			{
				PIXSetTargetWindow(WindowHandle);

				const FString CapturePath = MakeWinPixCaptureFilePath(DestFileName);

				PIXCaptureParameters Parameters{};
				Parameters.GpuCaptureParameters.FileName = *CapturePath;

				PIXBeginCapture2(PIX_CAPTURE_GPU, &Parameters);
			}
		}

		void EndCapture()
		{
			if (WinPixGpuCapturerHandle)
			{
				PIXEndCapture(0);
			}
		}

		void OpenCapture(const FString& FileName)
		{
			if (WinPixGpuCapturerHandle)
			{
				PIXOpenCaptureInUI(*FileName);
			}
		}

	private:
		void* WinPixGpuCapturerHandle{};
	};
#endif

	/** Dummy input device that is used only to generate a Tick. */
	class FPixDummyInputDevice : public IInputDevice
	{
	public:
		FPixDummyInputDevice(FPixWinPluginModule* InModule)
			: Module(InModule)
		{
		}

		virtual void Tick(float DeltaTime) override
		{
			if (ensure(Module != nullptr))
			{
				Module->Tick(DeltaTime);
			}
		}

		virtual void SendControllerEvents() override { }
		virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override { }
		virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override { return(false); }
		virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override { }
		virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues& values) override { }

	private:
		FPixWinPluginModule* Module;
	};
}

void FPixWinPluginModule::StartupModule()
{
#if PIX_PLUGIN_ENABLED
	PixGraphicsAnalysisInterface = new Impl::FPixGraphicsAnalysisInterface();
	if (PixGraphicsAnalysisInterface->IsValid())
	{
		FString CapturePath = FPaths::ProjectSavedDir() / TEXT("PixCaptures");
		if (!IFileManager::Get().DirectoryExists(*CapturePath))
		{
			IFileManager::Get().MakeDirectory(*CapturePath, true);
		}

		// Register modular features.
		IModularFeatures::Get().RegisterModularFeature(IRenderCaptureProvider::GetModularFeatureName(), (IRenderCaptureProvider*)this);
		IModularFeatures::Get().RegisterModularFeature(IInputDeviceModule::GetModularFeatureName(), (IInputDeviceModule*)this);

		// Register console command.
		ConsoleCommandCaptureFrame = new FAutoConsoleCommand(
			TEXT("pix.GpuCaptureFrame"),
			TEXT("Captures the rendering commands of the next frame."),
			FConsoleCommandDelegate::CreateRaw(this, &FPixWinPluginModule::CaptureFrame));

		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FPixWinPluginModule::OnPostEngineInit);

		UE_LOGF(PixWinPlugin, Log, "PIX capture plugin is ready!");
	}
	else
#endif
	{
		UE_LOGF(PixWinPlugin, Log, "PIX capture plugin failed to initialize! Check that the process is launched from PIX.");
	}
}

void FPixWinPluginModule::ShutdownModule()
{
#if PIX_PLUGIN_ENABLED
	delete PixGraphicsAnalysisInterface;
	PixGraphicsAnalysisInterface = nullptr;
	delete ConsoleCommandCaptureFrame;
	ConsoleCommandCaptureFrame = nullptr;

	IModularFeatures::Get().UnregisterModularFeature(IRenderCaptureProvider::GetModularFeatureName(), (IRenderCaptureProvider*)this);
	IModularFeatures::Get().UnregisterModularFeature(IInputDeviceModule::GetModularFeatureName(), (IInputDeviceModule*)this);

#if WITH_EDITOR
	EditorExtension.Reset();
#endif

#endif
}

TSharedPtr<IInputDevice> FPixWinPluginModule::CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	UE_LOGF(PixWinPlugin, Log, "Creating dummy input device (for intercepting engine ticks)");
	return MakeShared<Impl::FPixDummyInputDevice>(this);
}

void FPixWinPluginModule::CaptureFrame(FViewport* InViewport, uint32 InFlags, const FString& InDestFileName)
{
	if (!bEndCaptureNextTick)
	{
		DoFrameCaptureCurrentViewport(InViewport, InFlags, InDestFileName);
	}
}

void FPixWinPluginModule::BeginCapture(FRHICommandListImmediate* InRHICommandList, uint32 InFlags, const FString& InDestFileName)
{
#if PIX_PLUGIN_ENABLED
	CurrentCaptureDestFileName = MakeWinPixCaptureFilePath(InDestFileName);
	CurrentCaptureFlags = InFlags;
	InRHICommandList->SubmitAndBlockUntilGPUIdle();
	InRHICommandList->EnqueueLambda([Pix = PixGraphicsAnalysisInterface, DestFileName = CurrentCaptureDestFileName](FRHICommandListImmediate& RHICommandList)
	{
		Pix->BeginCapture(nullptr, DestFileName);
	});
#endif
}

void FPixWinPluginModule::EndCapture(FRHICommandListImmediate* InRHICommandList)
{
#if PIX_PLUGIN_ENABLED
	InRHICommandList->SubmitAndBlockUntilGPUIdle();
	InRHICommandList->EnqueueLambda(
		[Pix = PixGraphicsAnalysisInterface, 
		Flags = CurrentCaptureFlags, 
		DestFileName = MoveTemp(CurrentCaptureDestFileName)](FRHICommandListImmediate& RHICommandList)
	{
		Pix->EndCapture();

		// If we're already attached, don't open a new PIX instance
		if ((Flags & IRenderCaptureProvider::ECaptureFlags_Launch) && !Pix->IsAttached())
		{
			Pix->OpenCapture(DestFileName);
		}
	});
	CurrentCaptureFlags = 0u;
	CurrentCaptureDestFileName.Empty();
#endif
}

void FPixWinPluginModule::OnPostEngineInit()
{
#if WITH_EDITOR
	if (FSlateApplication::IsInitialized() && !IsRunningCommandlet())
	{
		EditorExtension = MakeShared<FPixWinPluginEditorExtension>(this);
	}
#endif // WITH_EDITOR
}

void FPixWinPluginModule::Tick(float DeltaTime)
{
#if PIX_PLUGIN_ENABLED
	const bool bNewCurrentlyAttached = PixGraphicsAnalysisInterface && PixGraphicsAnalysisInterface->IsAttached();
	if (bNewCurrentlyAttached != bCurrentlyAttached)
	{
		// Call EnableIdealGPUCaptureOptions the first time we are attached
		if (bNewCurrentlyAttached)
		{
			FDynamicRHI::EnableIdealGPUCaptureOptions(true);
		}

		bCurrentlyAttached = bNewCurrentlyAttached;
	}

	if (bBeginCaptureNextTick)
	{
		// Start a capture.
		bBeginCaptureNextTick = false;
		bEndCaptureNextTick = true;

		BeginFrameCapture(nullptr, FString());
	}
	else if (bEndCaptureNextTick)
	{
		// End a capture.
		bEndCaptureNextTick = false;

		EndFrameCapture(0, FString());
	}
#endif
}

void FPixWinPluginModule::DoFrameCaptureCurrentViewport(FViewport* InViewport, uint32 InFlags, const FString& InDestFileName)
{
#if PIX_PLUGIN_ENABLED
	// infer the intended viewport to intercept/capture:
	FViewport* Viewport = InViewport;

	check(GEngine);
	if (!Viewport && GEngine->GameViewport)
	{
		check(GEngine->GameViewport->Viewport);
		if (GEngine->GameViewport->Viewport->HasFocus())
		{
			Viewport = GEngine->GameViewport->Viewport;
		}
	}

#if WITH_EDITOR
	if (!Viewport && GEditor)
	{
		// WARNING: capturing from a "PIE-Eject" Editor viewport will not work as
		// expected; in such case, capture via the console command
		// (this has something to do with the 'active' editor viewport when the UI
		// button is clicked versus the one which the console is attached to)
		Viewport = GEditor->GetActiveViewport();
		if (!Viewport)
		{
			const TArray<class FEditorViewportClient*>& ViewportClients = GEditor->GetAllViewportClients();
			if (ViewportClients.Num() > 0)
			{
				Viewport = ViewportClients[0]->Viewport;
			}
		}
	}
#endif // WITH_EDITOR

	const FString DestFileName = MakeWinPixCaptureFilePath(InDestFileName);

	check(Viewport);
	BeginFrameCapture(Viewport->GetWindow(), DestFileName);

	Viewport->Draw(true);

	EndFrameCapture(InFlags, DestFileName);
#endif
}

void FPixWinPluginModule::BeginFrameCapture(void* HWnd, const FString& DestFileName)
{
#if PIX_PLUGIN_ENABLED
	UE_LOGF(PixWinPlugin, Log, "Capturing a frame in PIX");

	Impl::FPixGraphicsAnalysisInterface* Pix = PixGraphicsAnalysisInterface;
	HWND WindowHandle = HWnd ? (HWND)HWnd : GetActiveWindow();

	ENQUEUE_RENDER_COMMAND(PixWinBeginFrameCapture)(
		[Pix, WindowHandle, DestFileName](FRHICommandListImmediate& RHICommandList)
		{
			if (Pix && Pix->IsValid())
			{
				Pix->BeginCapture(WindowHandle, FString());
			}
		});
#endif
}

void FPixWinPluginModule::EndFrameCapture(uint32 InFlags, const FString& DestFileName)
{
#if PIX_PLUGIN_ENABLED
	Impl::FPixGraphicsAnalysisInterface* Pix = PixGraphicsAnalysisInterface;
	ENQUEUE_RENDER_COMMAND(PixWinEndFrameCapture)(
		[Pix, InFlags, DestFileName](FRHICommandListImmediate& RHICommandList)
		{
			if (Pix && Pix->IsValid())
			{
				RHICommandList.SubmitAndBlockUntilGPUIdle();
				Pix->EndCapture();

				// If we're already attached, don't open a new PIX instance
				if ((InFlags & IRenderCaptureProvider::ECaptureFlags_Launch) && !Pix->IsAttached())
				{
					Pix->OpenCapture(DestFileName);
				}
			}
		});
#endif
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FPixWinPluginModule, PixWinPlugin)

