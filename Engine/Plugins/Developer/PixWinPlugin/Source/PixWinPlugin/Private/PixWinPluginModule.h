// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPixWinPlugin.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(PixWinPlugin, Log, All);

class FAutoConsoleCommand;
class FPixWinPluginEditorExtension;
class IInputDevice;

namespace Impl { class FPixDummyInputDevice; }
namespace Impl { class FPixGraphicsAnalysisInterface; }

/** PIX capture plugin implementation. */
class FPixWinPluginModule : public IPixWinPlugin
{
	friend Impl::FPixDummyInputDevice;

public:
	// Begin IRenderCaptureProvider interface.
	virtual void CaptureFrame(FViewport* InViewport, uint32 InFlags, const FString& InDestFileName) override;
	virtual void BeginCapture(FRHICommandListImmediate* InRHICommandList, uint32 InFlags, const FString& InDestFileName) override;
	virtual void EndCapture(FRHICommandListImmediate* InRHICommandList) override;
	// End IRenderCaptureProvider interface.

protected:
	// Begin IModuleInterface interface.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface interface.

	// Begin IInputDeviceModule interface.
	virtual TSharedPtr<IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
	// End IInputDeviceModule interface.

private:
	void OnPostEngineInit();

	/** Tick to handle full frame capture events. */
	void Tick(float DeltaTime);

	void DoFrameCaptureCurrentViewport(FViewport* InViewport, uint32 InFlags, const FString& InDestFileName);

	void BeginFrameCapture(void* HWnd, const FString& DestFileName);
	void EndFrameCapture(uint32 InFlags, const FString& DestFileName);

	/** Helper function for CVar command binding. */
	void CaptureFrame() { return CaptureFrame(nullptr, 0, FString()); }

	Impl::FPixGraphicsAnalysisInterface* PixGraphicsAnalysisInterface = nullptr;
	FAutoConsoleCommand* ConsoleCommandCaptureFrame = nullptr;

	bool bBeginCaptureNextTick = false;
	bool bEndCaptureNextTick = false;

	bool bCurrentlyAttached = false;

	FString CurrentCaptureDestFileName;
	uint32 CurrentCaptureFlags = 0u;

#if WITH_EDITOR
	TSharedPtr<FPixWinPluginEditorExtension> EditorExtension;
#endif
};
