// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PixWinPluginStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "HAL/FileManager.h"
#include "Interfaces/IPluginManager.h"

FString FPixWinPluginStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	TSharedPtr<IPlugin> ThisPlugin = IPluginManager::Get().FindPlugin(TEXT("PixWinPlugin"));
	check(ThisPlugin.IsValid());
	static FString ContentDir = ThisPlugin->GetBaseDir() / TEXT("Resources");
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr<FSlateStyleSet> FPixWinPluginStyle::StyleSet = nullptr;
TSharedPtr<class ISlateStyle> FPixWinPluginStyle::Get() { return StyleSet; }

void FPixWinPluginStyle::Initialize()
{
	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet("PixWinPluginStyle"));

	FString ProjectResourceDir = FPaths::ProjectPluginsDir() / TEXT("PixWinPlugin") / TEXT("Resources");
	FString EngineResourceDir = FPaths::EnginePluginsDir() / TEXT("PixWinPlugin") / TEXT("Resources");

	if (IFileManager::Get().DirectoryExists(*ProjectResourceDir)) //Is the plugin in the project? In that case, use those resources
	{
		StyleSet->SetContentRoot(ProjectResourceDir);
		StyleSet->SetCoreContentRoot(ProjectResourceDir);
	}
	else //Otherwise, use the global ones
	{
		StyleSet->SetContentRoot(EngineResourceDir);
		StyleSet->SetCoreContentRoot(EngineResourceDir);
	}

	StyleSet->Set("PixWinPlugin.Icon", new FSlateImageBrush(FPixWinPluginStyle::InContent("Icon40", ".png"), FVector2D(40.0f, 40.0f)));
	StyleSet->Set("PixWinPlugin.CaptureFrame", new FSlateImageBrush(FPixWinPluginStyle::InContent("ViewportIcon16", ".png"), FVector2D(16.0f, 16.0f)));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

void FPixWinPluginStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

#endif //WITH_EDITOR
