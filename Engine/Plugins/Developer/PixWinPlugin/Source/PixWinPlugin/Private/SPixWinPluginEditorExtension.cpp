// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPixWinPluginEditorExtension.h"

#if WITH_EDITOR

#include "PixWinPluginCommands.h"
#include "PixWinPluginModule.h"
#include "PixWinPluginStyle.h"

#include "Editor/EditorEngine.h"
#include "SEditorViewportToolBarMenu.h"
#include "SViewportToolBarComboMenu.h"
#include "Kismet2/DebuggerCommands.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Interfaces/IMainFrameModule.h"
#include "LevelEditor.h"
#include "RHI.h"
#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"

extern UNREALED_API UEditorEngine* GEditor;

FPixWinPluginEditorExtension::FPixWinPluginEditorExtension(FPixWinPluginModule* ThePlugin)
{
	Initialize(ThePlugin);
}

FPixWinPluginEditorExtension::~FPixWinPluginEditorExtension()
{
	FPixWinPluginStyle::Shutdown();
	FPixWinPluginCommands::Unregister();
	UToolMenus::UnregisterOwner(this);
}

void FPixWinPluginEditorExtension::Initialize(FPixWinPluginModule* ThePlugin)
{
	if (GUsingNullRHI)
	{
		UE_LOGF(PixWinPlugin, Display, "PixWin Plugin will not be loaded because a Null RHI (Cook Server, perhaps) is being used.");
		return;
	}

	// The LoadModule request below will crash if running as an editor commandlet!
	check(!IsRunningCommandlet());

	FPixWinPluginStyle::Initialize();
	FPixWinPluginCommands::Register();

#if WITH_EDITOR
	if (!IsRunningGame())
	{
		FToolMenuOwnerScoped ScopedOwner(this);

		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ViewportToolbar");
	
		FToolMenuSection& RightSection = Menu->FindOrAddSection("Right");
		FToolMenuEntry& Entry = RightSection.AddMenuEntry("PixWinCaptureFrame", FPixWinPluginCommands::Get().CaptureFrame);
		Entry.ToolBarData.LabelOverride = FText::GetEmpty();
		Entry.InsertPosition.Position = EToolMenuInsertType::First;
	}
#endif // WITH_EDITOR

	// Would be nice to use the preprocessor definition WITH_EDITOR instead, but the user may launch a standalone the game through the editor...
	if (GEditor != nullptr)
	{
		check(FPlayWorldCommands::GlobalPlayWorldActions.IsValid());

		//Register the editor hotkeys
		FPlayWorldCommands::GlobalPlayWorldActions->MapAction(FPixWinPluginCommands::Get().CaptureFrame,
			FExecuteAction::CreateLambda([]()
				{
					FPixWinPluginModule& PluginModule = FModuleManager::GetModuleChecked<FPixWinPluginModule>("PixWinPlugin");
					PluginModule.CaptureFrame(nullptr, IRenderCaptureProvider::ECaptureFlags_Launch, FString());
				}),
			FCanExecuteAction());
	}
}

#endif //WITH_EDITOR
