// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "SViewportToolBar.h"

class FPixWinPluginModule;
class FExtensionBase;
class FExtensibilityManager;
class FExtender;
class FToolBarBuilder;

class FPixWinPluginEditorExtension
{
public:
	FPixWinPluginEditorExtension(FPixWinPluginModule* ThePlugin);
	~FPixWinPluginEditorExtension();

private:
	void Initialize(FPixWinPluginModule* ThePlugin);
};

#endif //WITH_EDITOR
