// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInputDeviceModule.h"
#include "IRenderCaptureProvider.h"
#include "Modules/ModuleManager.h"

/** PIX capture plugin interface. */
class IPixWinPlugin : public IInputDeviceModule, public IRenderCaptureProvider
{
public:
	static inline IPixWinPlugin& Get()
	{
		return FModuleManager::LoadModuleChecked<IPixWinPlugin>("PixWinPlugin");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("PixWinPlugin");
	}
};
